// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioCapture_VisionOS : AudioCapture
	{
		public AudioCapture_VisionOS(ReadOnlyTargetRules Target) : base(Target)
		{
			if (Target.Platform == UnrealTargetPlatform.VisionOS)
            {
                PrivateDependencyModuleNames.Add("AudioCaptureAudioUnit");
            }
        }
	}
}