{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Audio Capture",
	"Description" : "Plugin provides an interface for microphone input capture.",
	"Category" : "Audio",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name" : "AudioCapture",
			"Type": "Runtime",
			"ProgramAllowList": ["LiveLinkHub"],
			"LoadingPhase": "Default"
 		},
		{
			"Name": "AudioCaptureEditor",
			"Type": "UncookedOnly",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}
