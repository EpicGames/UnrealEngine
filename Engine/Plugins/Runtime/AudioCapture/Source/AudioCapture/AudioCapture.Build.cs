// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioCapture : ModuleRules
	{
		public AudioCapture(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AudioMixer",
					"AudioCaptureCore"
				}
			);

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PrivateDependencyModuleNames.Add("AudioCaptureWasapi");
				PrivateDependencyModuleNames.Add("AudioCaptureRtAudio");
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PrivateDependencyModuleNames.Add("AudioCaptureRtAudio");
			}
			else if (Target.Platform == UnrealTargetPlatform.IOS)
            {
                PrivateDependencyModuleNames.Add("AudioCaptureAudioUnit");
            }
            else if (Target.Platform == UnrealTargetPlatform.Android)
            {
                PrivateDependencyModuleNames.Add("AudioCaptureAndroid");
            }
        }
	}
}