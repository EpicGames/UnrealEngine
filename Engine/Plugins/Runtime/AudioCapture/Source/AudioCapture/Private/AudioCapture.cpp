// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCapture.h"

#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioCapture)

DEFINE_LOG_CATEGORY(LogAudioCapture);

UAudioCapture::UAudioCapture()
{

}

UAudioCapture::~UAudioCapture()
{

}

bool UAudioCapture::OpenDefaultAudioStream()
{
	if (!AudioCapture.IsStreamOpen())
	{
		Audio::FOnAudioCaptureFunction OnCapture = [this](const void* AudioData, int32 NumFrames, int32 InNumChannels, int32 InSampleRate, double StreamTime, bool bOverFlow)
		{
			OnGeneratedAudio((const float*)AudioData, NumFrames * InNumChannels);
		};

		// Start the stream here to avoid hitching the audio render thread. 
		Audio::FAudioCaptureDeviceParams Params;
		if (AudioCapture.OpenAudioCaptureStream(Params, MoveTemp(OnCapture), 1024))
		{
			// If we opened the capture stream succesfully, get the capture device info and initialize the UAudioGenerator
			Audio::FCaptureDeviceInfo Info;
			if (AudioCapture.GetCaptureDeviceInfo(Info))
			{
				Init(Info.PreferredSampleRate, Info.InputChannels);
				return true;
			}
		}
	}
	return false;
}

bool UAudioCapture::GetAudioCaptureDeviceInfo(FAudioCaptureDeviceInfo& OutInfo)
{
	Audio::FCaptureDeviceInfo Info;
	if (AudioCapture.GetCaptureDeviceInfo(Info))
	{
		OutInfo.DeviceName = FName(*Info.DeviceName);
		OutInfo.NumInputChannels = Info.InputChannels;
		OutInfo.SampleRate = Info.PreferredSampleRate;
		return true;
	}
	return false;
}

void UAudioCapture::StartCapturingAudio()
{
	if (AudioCapture.IsStreamOpen())
	{
		AudioCapture.StartStream();
	}
}

void UAudioCapture::StopCapturingAudio()
{
	if (AudioCapture.IsStreamOpen())
	{
		AudioCapture.StopStream();
	}

}

bool UAudioCapture::IsCapturingAudio()
{
	return AudioCapture.IsCapturing();
}

UAudioCapture* UAudioCaptureFunctionLibrary::CreateAudioCapture()
{
	LLM_SCOPE(ELLMTag::Audio);
	UAudioCapture* NewAudioCapture = NewObject<UAudioCapture>();
	if (NewAudioCapture->OpenDefaultAudioStream())
	{
		return NewAudioCapture;
	}
	
	UE_LOGF(LogAudioCapture, Error, "Failed to open a default audio stream to the audio capture device.");
	return nullptr;
}

void FAudioCaptureModule::StartupModule()
{
	LLM_SCOPE(ELLMTag::Audio);
	// Load platform specific implementations for audio capture (if specified in a .ini file)
	FString AudioCaptureModuleName;
	if (GConfig->GetString(TEXT("Audio"), TEXT("AudioCaptureModuleName"), AudioCaptureModuleName, GEngineIni))
	{
		FModuleManager::Get().LoadModule(*AudioCaptureModuleName);
	}
}

void FAudioCaptureModule::ShutdownModule()
{
}


IMPLEMENT_MODULE(FAudioCaptureModule, AudioCapture);
