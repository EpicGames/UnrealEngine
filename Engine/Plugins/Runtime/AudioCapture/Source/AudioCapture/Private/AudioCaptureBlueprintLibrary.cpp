// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureBlueprintLibrary.h"
#include "AudioCaptureCore.h"
#include "AudioThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioCaptureBlueprintLibrary)

FAudioInputDeviceInfo::FAudioInputDeviceInfo(const Audio::FCaptureDeviceInfo& InDeviceInfo) :
	DeviceName(InDeviceInfo.DeviceName),
	DeviceId(InDeviceInfo.DeviceId),
	InputChannels(InDeviceInfo.InputChannels),
	PreferredSampleRate(InDeviceInfo.PreferredSampleRate),
	bSupportsHardwareAEC(InDeviceInfo.bSupportsHardwareAEC)
{
}

FString UAudioCaptureBlueprintLibrary::Conv_AudioInputDeviceInfoToString(const FAudioInputDeviceInfo& InDeviceInfo)
{
	FString output = FString::Printf(TEXT("Device Name: %s, \nDevice Id: %s, \nNum Channels: %u, \nSample Rate: %u, \nSupports Hardware AEC: %u, \n"),
		*InDeviceInfo.DeviceName, *InDeviceInfo.DeviceId, InDeviceInfo.InputChannels, InDeviceInfo.PreferredSampleRate, InDeviceInfo.bSupportsHardwareAEC);

	return output;
}


void UAudioCaptureBlueprintLibrary::GetAvailableAudioInputDevices(const UObject* WorldContextObject, const FOnAudioInputDevicesObtained& OnObtainDevicesEvent)
{
	if (!IsInAudioThread())
	{
		//Send this over to the audio thread, with the same settings
		FAudioThread::RunCommandOnAudioThread([WorldContextObject, OnObtainDevicesEvent]()
			{
				GetAvailableAudioInputDevices(WorldContextObject, OnObtainDevicesEvent);
			});

		return;
	}

	TArray<FAudioInputDeviceInfo> AvailableDeviceInfos; //The array of audio device info to return

	Audio::FAudioCapture AudioCapture;
	TArray<Audio::FCaptureDeviceInfo> InputDevices;

	AudioCapture.GetCaptureDevicesAvailable(InputDevices);

	for (auto Iter = InputDevices.CreateConstIterator(); Iter; ++Iter)
	{
		AvailableDeviceInfos.Add(FAudioInputDeviceInfo(*Iter));
	}

	//Call delegate event, and send the info there
	OnObtainDevicesEvent.Execute(AvailableDeviceInfos);
}
