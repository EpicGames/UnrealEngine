// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureComponent.h"
#include "Audio.h"
#include "AudioAnalytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioCaptureComponent)

static const unsigned int MaxBufferSize = 2 * 5 * 48000;

UAudioCaptureComponent::UAudioCaptureComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSuccessfullyInitialized = false;
	bIsCapturing = false;
	CapturedAudioDataSamples = 0;
	ReadSampleIndex = 0;
	bIsDestroying = false;
	bIsNotReadyForForFinishDestroy = false;
	bIsStreamOpen = false;
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		CaptureAudioData.Reserve(2 * 48000 * 5);
	}
}

bool UAudioCaptureComponent::Init(int32& SampleRate)
{
	Audio::FCaptureDeviceInfo DeviceInfo;
	if (CaptureSynth.GetDefaultCaptureDeviceInfo(DeviceInfo))
	{
		if (DeviceInfo.PreferredSampleRate > 0)
		{
			SampleRate = DeviceInfo.PreferredSampleRate;
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Attempted to open a capture device with the invalid SampleRate value of %i", DeviceInfo.PreferredSampleRate);
		}
		NumChannels = DeviceInfo.InputChannels;

		if (NumChannels > 0 && NumChannels <= 8)
		{
			// This may fail if capture synths aren't supported on a given platform or if something went wrong with the capture device
			bIsStreamOpen = CaptureSynth.OpenDefaultStream();

			Audio::Analytics::RecordEvent_Usage(TEXT("AudioCapture.AudioCaptureComponentInitialized"));

			return true;
		}
		else
		{
			UE_LOGF(LogAudio, Warning, "Attempted to open a device with the invalid NumChannels value of %i", NumChannels);
		}

	}
	return false;
}

void UAudioCaptureComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// Flag that we're beginning to be destroyed
	// This is so that if a mic capture is open, we shut it down on the render thread
	bIsDestroying = true;

	// Make sure stop is kicked off
	Stop();
}

bool UAudioCaptureComponent::IsReadyForFinishDestroy()
{
	//Prevent an infinite loop here if it was marked pending kill while generating
	OnEndGenerate();

	return !bIsNotReadyForForFinishDestroy;
}

void UAudioCaptureComponent::FinishDestroy()
{
	if (CaptureSynth.IsStreamOpen())
	{
		CaptureSynth.AbortCapturing();
	}

	check(!CaptureSynth.IsStreamOpen());

	Super::FinishDestroy();
	bSuccessfullyInitialized = false;
	bIsCapturing = false;
	bIsDestroying = false;
	bIsStreamOpen = false;
}

void UAudioCaptureComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(CaptureAudioData.GetAllocatedSize());
}

void UAudioCaptureComponent::OnBeginGenerate()
{
	CapturedAudioDataSamples = 0;
	ReadSampleIndex = 0;
	CaptureAudioData.Reset();

	if (!bIsStreamOpen)
	{
		bIsStreamOpen = CaptureSynth.OpenDefaultStream();
	}

	if (bIsStreamOpen)
	{
		CaptureSynth.StartCapturing();
		check(CaptureSynth.IsCapturing());

		// Don't allow this component to be destroyed until the stream is closed again
		bIsNotReadyForForFinishDestroy = true;
		FramesSinceStarting = 0;
		ReadSampleIndex = 0;
	}

}

void UAudioCaptureComponent::OnEndGenerate()
{
	if (bIsStreamOpen)
	{
		check(CaptureSynth.IsStreamOpen());
		CaptureSynth.StopCapturing();
		bIsStreamOpen = false;

		bIsNotReadyForForFinishDestroy = false;
	}
}

int32 UAudioCaptureComponent::OnGenerateAudio(float* OutAudio, int32 NumSamples)
{
	// Don't do anything if the stream isn't open
	if (!bIsStreamOpen || !CaptureSynth.IsStreamOpen() || !CaptureSynth.IsCapturing())
	{
		// Just return NumSamples, which uses zero'd buffer
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
		return NumSamples;
	}
	int32 OutputSamplesGenerated = 0;
	//In case of severe overflow, just drop the data
	if (CaptureAudioData.Num() > MaxBufferSize)
	{
		//Clear the CaptureSynth's data, too
		CaptureSynth.GetAudioData(CaptureAudioData);
		CaptureAudioData.Reset();
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
		return NumSamples;
	}
	if (CapturedAudioDataSamples > 0 || CaptureSynth.GetNumSamplesEnqueued() > 1024)
	{
		// Check if we need to read more audio data from capture synth
		if (ReadSampleIndex + NumSamples > CaptureAudioData.Num())
		{
			// but before we do, copy off the remainder of the capture audio data buffer if there's data in it
			int32 SamplesLeft = FMath::Max(0, CaptureAudioData.Num() - ReadSampleIndex);
			if (SamplesLeft > 0)
			{
				float* CaptureDataPtr = CaptureAudioData.GetData();
				if (!(ReadSampleIndex + NumSamples > MaxBufferSize - 1))
				{
					FMemory::Memcpy(OutAudio, &CaptureDataPtr[ReadSampleIndex], SamplesLeft * sizeof(float));
				}
				else
				{
					UE_LOGF(LogAudio, Warning, "Attempt to write past end of buffer in OnGenerateAudio, when we got more data from the synth");
					return NumSamples;
				}
				// Track samples generated
				OutputSamplesGenerated += SamplesLeft;
			}
			// Get another block of audio from the capture synth
			CaptureAudioData.Reset();
			CaptureSynth.GetAudioData(CaptureAudioData);
			// Reset the read sample index since we got a new buffer of audio data
			ReadSampleIndex = 0;
		}
		// note it's possible we didn't get any more audio in our last attempt to get it
		if (CaptureAudioData.Num() > 0)
		{
			// Compute samples to copy
			int32 NumSamplesToCopy = FMath::Min(NumSamples - OutputSamplesGenerated, CaptureAudioData.Num() - ReadSampleIndex);
			float* CaptureDataPtr = CaptureAudioData.GetData();
			if (!(ReadSampleIndex + NumSamplesToCopy > MaxBufferSize - 1))
			{
				FMemory::Memcpy(&OutAudio[OutputSamplesGenerated], &CaptureDataPtr[ReadSampleIndex], NumSamplesToCopy * sizeof(float));
			}
			else
			{
				UE_LOGF(LogAudio, Warning, "Attempt to read past end of buffer in OnGenerateAudio, when we did not get more data from the synth");
				return NumSamples;
			}
			ReadSampleIndex += NumSamplesToCopy;
			OutputSamplesGenerated += NumSamplesToCopy;
		}
		CapturedAudioDataSamples += OutputSamplesGenerated;
	}
	else
	{
		// Say we generated the full samples, this will result in silence
		FMemory::Memzero(OutAudio, NumSamples * sizeof(float));
		OutputSamplesGenerated = NumSamples;
	}

	if (OutputSamplesGenerated < NumSamples)
	{
		FMemory::Memzero(&OutAudio[OutputSamplesGenerated], (NumSamples - OutputSamplesGenerated) * sizeof(float));
	}
	return OutputSamplesGenerated;
}
