// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "HAL/ThreadSafeBool.h"
#include "DSP/Delay.h"
#include "DSP/EnvelopeFollower.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Generators/AudioGenerator.h"
#include "AudioCaptureCore.h"

#include "AudioCapture.generated.h"

#define UE_API AUDIOCAPTURE_API

DECLARE_LOG_CATEGORY_EXTERN(LogAudioCapture, Log, All);

class FAudioCaptureModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

// Struct defining the time synth global quantization settings
USTRUCT(BlueprintType)
struct FAudioCaptureDeviceInfo
{
	GENERATED_USTRUCT_BODY()

	// The name of the audio capture device
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AudioCapture")
	FName DeviceName;

	// The number of input channels
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AudioCapture")
	int32 NumInputChannels = 0;

	// The sample rate of the audio capture device
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AudioCapture")
	int32 SampleRate = 0;
};

// Class which opens up a handle to an audio capture device.
// Allows other objects to get audio buffers from the capture device.
UCLASS(MinimalAPI, ClassGroup = (Custom), meta = (BlueprintSpawnableComponent))
class UAudioCapture : public UAudioGenerator
{
	GENERATED_BODY()

public:
	UE_API UAudioCapture();
	UE_API ~UAudioCapture();

	UE_API bool OpenDefaultAudioStream();

	// Returns the audio capture device info
	UFUNCTION(BlueprintCallable, Category = "AudioCapture")
	UE_API bool GetAudioCaptureDeviceInfo(FAudioCaptureDeviceInfo& OutInfo);

	// Starts capturing audio
	UFUNCTION(BlueprintCallable, Category = "AudioCapture")
	UE_API void StartCapturingAudio();

	// Stops capturing audio
	UFUNCTION(BlueprintCallable, Category = "AudioCapture")
	UE_API void StopCapturingAudio();

	// Returns true if capturing audio
	UFUNCTION(BlueprintCallable, Category = "AudioCapture")
	UE_API bool IsCapturingAudio();

protected:

	Audio::FAudioCapture AudioCapture;
};

UCLASS(MinimalAPI)
class UAudioCaptureFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = "Audio Capture")
	static UE_API class UAudioCapture* CreateAudioCapture();
};

#undef UE_API
