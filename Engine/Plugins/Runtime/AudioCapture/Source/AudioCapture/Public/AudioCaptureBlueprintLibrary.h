// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SubmixEffects/AudioMixerSubmixEffectDynamicsProcessor.h"
#include "Sound/SoundEffectSource.h"
#include "Sound/AudioBus.h"
#include "SampleBuffer.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundSubmixSend.h"
#include "DSP/SpectrumAnalyzer.h"
#include "AudioMixer.h"
#include "AudioMixerTypes.h"
#include "AudioCapture.h"
#include "AudioCaptureBlueprintLibrary.generated.h"

#define UE_API AUDIOCAPTURE_API


/**
 * Platform audio input device info, in a Blueprint-readable format
 */
USTRUCT(BlueprintType)
struct FAudioInputDeviceInfo
{
	GENERATED_USTRUCT_BODY()

	FAudioInputDeviceInfo() :
		DeviceName(""),
		DeviceId(""),
		InputChannels(0),
		PreferredSampleRate(0),
		bSupportsHardwareAEC(true) 
	{};

	UE_API FAudioInputDeviceInfo(const Audio::FCaptureDeviceInfo& InDeviceInfo);

	/** The name of the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	FString DeviceName;

	/** ID of the device. */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	FString DeviceId;

	/** The number of channels supported by the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	int32 InputChannels;

	/** The preferred sample rate of the audio device */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	int32 PreferredSampleRate;

	/** Whether or not the device supports Acoustic Echo Canceling */
	UPROPERTY(BlueprintReadOnly, Category = "Audio")
	uint8 bSupportsHardwareAEC : 1;
};

/**
 * Called when a list of all available audio devices is retrieved
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FOnAudioInputDevicesObtained, const TArray<FAudioInputDeviceInfo>&, AvailableDevices);

UCLASS(MinimalAPI, meta = (ScriptName = "AudioCaptureLibrary"))
class UAudioCaptureBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	public:

	/**
	 * Returns the device info in a human readable format
	 * @param info - The audio device data to print
	 * @return The data in a string format
	 */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Audio Input Device Info To String", CompactNodeTitle = "To String", BlueprintAutocast), Category = "Audio")
	static UE_API FString Conv_AudioInputDeviceInfoToString(const FAudioInputDeviceInfo& info);

	/**
	 * Gets information about all audio output devices available in the system
	 * @param OnObtainDevicesEvent - the event to fire when the audio endpoint devices have been retrieved
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio", meta = (WorldContext = "WorldContextObject"))
	static UE_API void GetAvailableAudioInputDevices(const UObject* WorldContextObject, const FOnAudioInputDevicesObtained& OnObtainDevicesEvent);
};

#undef UE_API
