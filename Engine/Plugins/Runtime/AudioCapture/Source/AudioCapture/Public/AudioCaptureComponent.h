// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AudioCapture.h"
#include "Components/SynthComponent.h"
#include "AudioCaptureComponent.generated.h"

#define UE_API AUDIOCAPTURE_API

UCLASS(MinimalAPI, ClassGroup = Synth, meta = (BlueprintSpawnableComponent))
class UAudioCaptureComponent : public USynthComponent
{
	GENERATED_BODY()

protected:

	UE_API UAudioCaptureComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin USynthComponent interface
	UE_API virtual bool Init(int32& SampleRate) override;
	UE_API virtual int32 OnGenerateAudio(float* OutAudio, int32 NumSamples) override;
	UE_API virtual void OnBeginGenerate() override;
	UE_API virtual void OnEndGenerate() override;
	//~ End USynthComponent interface

	//~ Begin UObject interface
	UE_API virtual void BeginDestroy();
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UObject interface

public:
	/** 
	*   Induced latency in audio frames to use to account for jitter between mic capture hardware and audio render hardware. 
	 *	Increasing this number will increase latency but reduce potential for underruns.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Latency", meta = (ClampMin = "0", ClampMax = "1024"))
	int32 JitterLatencyFrames;

private:

	Audio::FAudioCaptureSynth CaptureSynth;
	TArray<float> CaptureAudioData;
	int32 CapturedAudioDataSamples;

	bool bSuccessfullyInitialized;
	bool bIsCapturing;
	bool bIsStreamOpen;
	int32 CaptureChannels;
	int32 FramesSinceStarting;
	int32 ReadSampleIndex;
	FThreadSafeBool bIsDestroying;
	FThreadSafeBool bIsNotReadyForForFinishDestroy;
};

#undef UE_API
