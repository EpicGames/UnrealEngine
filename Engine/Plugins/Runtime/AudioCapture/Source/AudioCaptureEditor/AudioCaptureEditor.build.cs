// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioCaptureEditor : ModuleRules
{
	public AudioCaptureEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"EditorFramework",
				"UnrealEd",
				"AudioEditor",
				"AudioCapture",
				"AudioCaptureCore",
				"AudioCaptureRtAudio"
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.Add("AudioCaptureWasapi");
			
			// RtAudio depends on DirectSound
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DirectSound");
		}
	}
}
