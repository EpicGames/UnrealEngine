// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureEditor.h"
#include "AudioRecordingManager.h"
#include "IAudioCaptureEditorModule.h"
#include "Modules/ModuleManager.h"


namespace Audio
{
	bool FAudioCaptureEditor::IsReadyToRecord()
	{
		return AudioRecordingManager.IsReadyToRecord();
	}

	bool FAudioCaptureEditor::IsRecording()
	{
		return AudioRecordingManager.IsRecording();
	}

	bool FAudioCaptureEditor::IsStopped()
	{
		return AudioRecordingManager.IsStopped();
	}

	void FAudioCaptureEditor::Start(const FTakeRecorderAudioSettings& InSettings)
	{
		FRecordingSettings RecordSettings;
		RecordSettings.AudioCaptureDeviceId = InSettings.AudioCaptureDeviceId;
		RecordSettings.AudioInputBufferSize = InSettings.AudioInputBufferSize;
		RecordSettings.RecordingDuration = InSettings.RecordingDuration;
		RecordSettings.NumRecordChannels = InSettings.NumRecordChannels;

		AudioRecordingManager.StartRecording(RecordSettings);
	}

	void FAudioCaptureEditor::Stop()
	{
		AudioRecordingManager.StopRecording();
	}

	TObjectPtr<USoundWave> FAudioCaptureEditor::GetRecordedSoundWave(const FTakeRecorderAudioSourceSettings& InSourceSettings)
	{
		FRecordingManagerSourceSettings RecorderSettings;
		RecorderSettings.AssetName = InSourceSettings.AssetName;
		RecorderSettings.Directory = InSourceSettings.Directory;
		RecorderSettings.GainDb = InSourceSettings.GainDb;
		RecorderSettings.InputChannelNumber = InSourceSettings.InputChannelNumber;
		RecorderSettings.StartTimecode = InSourceSettings.StartTimecode;
		RecorderSettings.VideoFrameRate = InSourceSettings.VideoFrameRate;

		return AudioRecordingManager.GetRecordedSoundWave(RecorderSettings);
	}

	bool FAudioCaptureEditor::GetCaptureDeviceInfo(FTakeRecorderAudioDeviceInfo& OutInfo, int32 InDeviceIndex)
	{
		FCaptureDeviceInfo DeviceInfo;
		bool bFoundDevice = AudioRecordingManager.GetCaptureDeviceInfo(DeviceInfo, InDeviceIndex);
		if (bFoundDevice)
		{
			CopyDeviceInfo(DeviceInfo, OutInfo);
		}

		return bFoundDevice;
	}

	bool FAudioCaptureEditor::GetCaptureDevicesAvailable(TArray<FTakeRecorderAudioDeviceInfo>& OutDevices)
	{
		TArray<FCaptureDeviceInfo> DeviceArray;
		bool bFoundDevice = AudioRecordingManager.GetCaptureDevicesAvailable(DeviceArray);
		if (bFoundDevice)
		{
			for (const FCaptureDeviceInfo& DeviceInfo : DeviceArray)
			{
				FTakeRecorderAudioDeviceInfo TempInfo;
				CopyDeviceInfo(DeviceInfo, TempInfo);

				OutDevices.Add(MoveTemp(TempInfo));
			}
		}

		return bFoundDevice;
	}

	void FAudioCaptureEditor::CopyDeviceInfo(const FCaptureDeviceInfo& InDeviceInfo, FTakeRecorderAudioDeviceInfo& OutDeviceInfo)
	{
		OutDeviceInfo.DeviceId = InDeviceInfo.DeviceId;
		OutDeviceInfo.DeviceName = InDeviceInfo.DeviceName;
		OutDeviceInfo.InputChannels = InDeviceInfo.InputChannels;
		OutDeviceInfo.PreferredSampleRate = InDeviceInfo.PreferredSampleRate;
	}

}
