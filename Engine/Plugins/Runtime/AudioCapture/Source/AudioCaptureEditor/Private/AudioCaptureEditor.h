// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioCaptureEditor.h"

#include "AudioCaptureDeviceInterface.h"
#include "AudioRecordingManager.h"

namespace Audio
{
	class FAudioCaptureEditor : public IAudioCaptureEditor
	{
	public:

		virtual ~FAudioCaptureEditor() = default;

		// Begin IAudioCaptureEditor overrides
		virtual bool IsReadyToRecord() override;
		virtual bool IsRecording() override;
		virtual bool IsStopped() override;
		virtual void Start(const FTakeRecorderAudioSettings& InSettings) override;
		virtual void Stop() override;
		virtual TObjectPtr<USoundWave> GetRecordedSoundWave(const FTakeRecorderAudioSourceSettings& InSourceSettings) override;
		virtual bool GetCaptureDeviceInfo(FTakeRecorderAudioDeviceInfo& OutInfo, int32 InDeviceIndex) override;
		virtual bool GetCaptureDevicesAvailable(TArray<FTakeRecorderAudioDeviceInfo>& OutDevices) override;
		// End IAudioCaptureEditor overrides

	private:

		void CopyDeviceInfo(const Audio::FCaptureDeviceInfo& InDeviceInfo, FTakeRecorderAudioDeviceInfo& OutDeviceInfo);

		Audio::FAudioRecordingManager AudioRecordingManager;
	};

}
