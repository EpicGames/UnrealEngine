// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureEditor.h"
#include "IAudioCaptureEditorModule.h"
#include "Modules/ModuleManager.h"


class FAudioCaptureEditorModule : public IAudioCaptureEditorModule
{
private:

	virtual void StartupModule() override
	{
		FactoryDelegate = RegisterAudioRecorder(
			[]() {
				return TUniquePtr<IAudioCaptureEditor>(new Audio::FAudioCaptureEditor());
			}
		);
	}

	virtual void ShutdownModule() override
	{
		UnregisterAudioRecorder(FactoryDelegate);
	}

	virtual FDelegateHandle RegisterAudioRecorder(const TFunction<TUniquePtr<IAudioCaptureEditor>()>& FactoryFunction) override
	{
		ensureMsgf(!AudioFactory, TEXT("Audio recorder already registered."));

		AudioFactory = FactoryFunction;
		AudioFactoryHandle = FDelegateHandle(FDelegateHandle::GenerateNewHandle);
		return AudioFactoryHandle;
	}

	virtual void UnregisterAudioRecorder(FDelegateHandle Handle) override
	{
		if (Handle == AudioFactoryHandle)
		{
			AudioFactory = nullptr;
			AudioFactoryHandle = FDelegateHandle();
		}
	}

	virtual bool HasAudioRecorder() const override
	{
		return AudioFactoryHandle.IsValid();
	}

	virtual TUniquePtr<IAudioCaptureEditor> CreateAudioRecorder() const override
	{
		return AudioFactory ? AudioFactory() : TUniquePtr<IAudioCaptureEditor>();
	}

	TFunction<TUniquePtr<IAudioCaptureEditor>()> AudioFactory;
	FDelegateHandle AudioFactoryHandle;

	FDelegateHandle FactoryDelegate;
};

IMPLEMENT_MODULE( FAudioCaptureEditorModule, AudioCaptureEditor );
