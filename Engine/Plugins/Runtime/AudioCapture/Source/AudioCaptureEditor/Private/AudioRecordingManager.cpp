// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioRecordingManager.h"
#include "AudioCaptureCore.h"
#include "AudioCaptureEditor.h"
#include "Sound/AudioSettings.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Factories/SoundFactory.h"
#include "Misc/EngineVersion.h"
#include "Misc/ScopeLock.h"
#include "Sound/SoundWave.h"
#include "AudioDeviceManager.h"
#include "Engine/Engine.h"
#include "Components/AudioComponent.h"
#include "DSP/Dsp.h"
#include "Audio.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY(LogMicManager);

namespace Audio
{

/**
* FAudioRecordingManager Implementation
*/

FAudioRecordingManager::FAudioRecordingManager()
	: RecorderState(EAudioRecorderState::PreRecord)
	, NumRecordedSamples(0)
	, NumFramesToRecord()
	, RecordingBlockSize(0)
	, RecordingSampleRate(44100.0f)
	, NumInputChannels(1)
{
}

FAudioRecordingManager::~FAudioRecordingManager()
{
	if (AudioCapture.IsStreamOpen())
	{
		AudioCapture.AbortStream();
	}
}

void FAudioRecordingManager::StartRecording(const FRecordingSettings& InSettings)
{
	EAudioRecorderState CurrentState = RecorderState.load();

	if (ensure(CurrentState == EAudioRecorderState::PreRecord))
	{
		RecorderState = EAudioRecorderState::Recording;

		// Default to 1024 blocks
		RecordingBlockSize = InSettings.AudioInputBufferSize;

		// If we have a stream open close it (reusing streams can cause a blip of previous recordings audio)
		if (AudioCapture.IsStreamOpen())
		{
			AudioCapture.StopStream();
			AudioCapture.CloseStream();
		}

		UE_LOGF(LogMicManager, Log, "Starting mic recording.");

		// Copy the settings
		Settings = InSettings;

		FCaptureDeviceInfo DeviceInfo;
		int32 DeviceIndex = INDEX_NONE;
		if (!GetCaptureDeviceInfo(DeviceInfo, DeviceIndex, Settings.AudioCaptureDeviceId))
		{
			UE_LOGF(LogMicManager, Error, "Unable to get audio capture device.");
			return;
		}

		RecordingSampleRate = DeviceInfo.PreferredSampleRate;
		NumInputChannels = DeviceInfo.InputChannels;

		if (NumInputChannels == 0)
		{
			UE_LOGF(LogMicManager, Warning, "No audio channels to record.");
			return;
		}

		// Initialize Sound Waves array
		RecordedSoundWaves.Init(nullptr, NumInputChannels);

		// Reserve enough space in our current recording buffer for 10 seconds of audio to prevent slowing down due to allocations
		if (Settings.RecordingDuration > 0.0f)
		{
			int32 NumSamplesToReserve = Settings.RecordingDuration * RecordingSampleRate * NumInputChannels;
			CurrentRecordedPCMData.Reset(NumSamplesToReserve);

			// Set a limit on the number of frames to record
			NumFramesToRecord = RecordingSampleRate * Settings.RecordingDuration;
		}
		else
		{
			// if not given a duration, resize array to 60 seconds of audio anyway
			int32 NumSamplesToReserve = 60 * RecordingSampleRate * NumInputChannels;
			CurrentRecordedPCMData.Reset(NumSamplesToReserve);

			// Set to be INDEX_NONE otherwise
			NumFramesToRecord = INDEX_NONE;
		}

		NumRecordedSamples = 0;
		NumOverflowsDetected = 0;

		FAudioCaptureDeviceParams StreamParams;
		StreamParams.NumInputChannels = NumInputChannels;
		StreamParams.SampleRate = RecordingSampleRate;
		StreamParams.PCMAudioEncoding = EPCMAudioEncoding::PCM_16;
		StreamParams.DeviceIndex = DeviceIndex;

		uint32 BufferFrames = FMath::Max(RecordingBlockSize, 256);

		UE_LOGF(LogMicManager, Log, "Initialized mic recording manager at %.4f hz sample rate, %d channels, and %d Recording Block Size", RecordingSampleRate, StreamParams.NumInputChannels, BufferFrames);

		FOnAudioCaptureFunction OnAudioCaptureCallback = [this](const void* InBuffer, int32 InNumFrames, int32 InNumChannels, int32 InSampleRate, double InStreamTime, bool bInOverflow)
		{
			OnAudioCapture(InBuffer, InNumFrames, InStreamTime, bInOverflow);
		};

		// Open up new audio stream
		bool bSuccess = AudioCapture.OpenAudioCaptureStream(StreamParams, OnAudioCaptureCallback, BufferFrames);

		if (bSuccess)
		{
			AudioCapture.StartStream();
		}
		else
		{
			RecorderState = EAudioRecorderState::ErrorDetected;
		}
	}
}

static void SampleRateConvert(float CurrentSR, float TargetSR, int32 NumChannels, const TArray<int16>& CurrentRecordedPCMData, int32 NumSamplesToConvert, TArray<int16>& OutConverted)
{
	int32 NumInputSamples = CurrentRecordedPCMData.Num();
	int32 NumOutputSamples = NumInputSamples * TargetSR / CurrentSR;

	OutConverted.Reset(NumOutputSamples);

	float SrFactor = (double)CurrentSR / TargetSR;
	float CurrentFrameIndexInterpolated = 0.0f;
	check(NumSamplesToConvert <= CurrentRecordedPCMData.Num());
	int32 NumFramesToConvert = NumSamplesToConvert / NumChannels;
	int32 CurrentFrameIndex = 0;

	for (;;)
	{
		int32 NextFrameIndex = CurrentFrameIndex + 1;
		if (CurrentFrameIndex >= NumFramesToConvert || NextFrameIndex >= NumFramesToConvert)
		{
			break;
		}

		for (int32 Channel = 0; Channel < NumChannels; ++Channel)
		{
			int32 CurrentSampleIndex = CurrentFrameIndex * NumChannels + Channel;
			int32 NextSampleIndex = CurrentSampleIndex + NumChannels;

			int16 CurrentSampleValue = CurrentRecordedPCMData[CurrentSampleIndex];
			int16 NextSampleValue = CurrentRecordedPCMData[NextSampleIndex];

			int16 NewSampleValue = FMath::Lerp(CurrentSampleValue, NextSampleValue, CurrentFrameIndexInterpolated);

			OutConverted.Add(NewSampleValue);
		}

		CurrentFrameIndexInterpolated += SrFactor;

		// Wrap the interpolated frame between 0.0 and 1.0 to maintain float precision
		while (CurrentFrameIndexInterpolated >= 1.0f)
		{
			CurrentFrameIndexInterpolated -= 1.0f;
			
			// Every time it wraps, we increment the frame index
			++CurrentFrameIndex;
		}
	}
}

void FAudioRecordingManager::StopRecording()
{
	EAudioRecorderState CurrentState = RecorderState.load();

	if (ensure(CurrentState == EAudioRecorderState::Recording))
	{
		RecorderState = EAudioRecorderState::Stopped;

		AudioCapture.StopStream();
		AudioCapture.CloseStream();

		if (CurrentRecordedPCMData.Num() > 0)
		{
			if (NumFramesToRecord != INDEX_NONE)
			{
				NumRecordedSamples = FMath::Min(NumFramesToRecord * NumInputChannels, CurrentRecordedPCMData.Num());
			}
			else
			{
				NumRecordedSamples = CurrentRecordedPCMData.Num();
			}

			UE_LOGF(LogMicManager, Log, "Stopping mic recording. Recorded %d frames of audio (%.4f seconds). Detected %d buffer overflows.",
				NumRecordedSamples,
				((float)NumRecordedSamples / RecordingSampleRate) / (float)NumInputChannels,
				NumOverflowsDetected);
		}
	}
}

TObjectPtr<USoundWave> FAudioRecordingManager::GetRecordedSoundWave(const FRecordingManagerSourceSettings& InSourceSettings)
{
	EAudioRecorderState CurrentState = RecorderState.load();

	if (ensure(CurrentState == EAudioRecorderState::Stopped))
	{
		if (PCMDataToSerialize == nullptr)
		{
			ProcessRecordedData(InSourceSettings);
		}

		const int32 ChannelIndex = InSourceSettings.InputChannelNumber - 1;
		if (RecordedSoundWaves.IsValidIndex(ChannelIndex) && RecordedSoundWaves[ChannelIndex] != nullptr)
		{
			return RecordedSoundWaves[ChannelIndex];
		}
		else if (NumRecordedSamples > 0 && ChannelIndex < NumInputChannels)
		{
			return CreateSoundWaveAsset(InSourceSettings);
		}
	}

	return nullptr;
}

void FAudioRecordingManager::ProcessRecordedData(const FRecordingManagerSourceSettings& InSourceSettings)
{
	// If our sample rate isn't 44100, then we need to do a SRC
	if (RecordingSampleRate != 44100.0f)
	{
		UE_LOGF(LogMicManager, Log, "Converting sample rate from %d hz to 44100 hz.", (int32)RecordingSampleRate);

		SampleRateConvert(RecordingSampleRate, (float)WAVE_FILE_SAMPLERATE, NumInputChannels, CurrentRecordedPCMData, NumRecordedSamples, ConvertedPCMData);

		// Update the recorded samples to the converted buffer samples
		NumRecordedSamples = ConvertedPCMData.Num();
		PCMDataToSerialize = &ConvertedPCMData;
	}
	else
	{
		// Just use the original recorded buffer to serialize
		PCMDataToSerialize = &CurrentRecordedPCMData;
	}

	DeinterleaveRecordedData();
}

void FAudioRecordingManager::DeinterleaveRecordedData()
{
	// De-interleaved buffer size will be the number of frames of audio
	int32 NumFrames = NumRecordedSamples / NumInputChannels;

	// Reset our deinterleaved audio buffer
	DeinterleavedAudio.Reset(NumInputChannels);

	// Get ptr to the interleaved buffer for speed in non-release builds
	int16* InterleavedBufferPtr = PCMDataToSerialize->GetData();

	// For every input channel, create a new buffer
	for (int32 Channel = 0; Channel < NumInputChannels; ++Channel)
	{
		// Prepare a new deinterleaved buffer
		DeinterleavedAudio.Add(FDeinterleavedAudio());
		FDeinterleavedAudio& DeinterleavedChannelAudio = DeinterleavedAudio[Channel];

		DeinterleavedChannelAudio.PCMData.Reset();
		DeinterleavedChannelAudio.PCMData.AddUninitialized(NumFrames);

		// Get a ptr to the buffer for speed
		int16* DeinterleavedBufferPtr = DeinterleavedChannelAudio.PCMData.GetData();

		// Copy every N channel to the deinterleaved buffer, starting with the current channel
		int32 CurrentSample = Channel;
		for (int32 Frame = 0; Frame < NumFrames; ++Frame)
		{
			// Simply copy the interleaved value to the deinterleaved value
			DeinterleavedBufferPtr[Frame] = InterleavedBufferPtr[CurrentSample];

			// Increment the stride according the num channels
			CurrentSample += NumInputChannels;
		}
	}
}

void FAudioRecordingManager::ApplyGain(const FRecordingManagerSourceSettings& InSourceSettings, const int32 InNumFrames)
{
	// Convert input gain decibels into linear scale
	float InputGain = Audio::ConvertToLinear(InSourceSettings.GainDb);

	// Scale by the linear gain if it's been set to something
	if (InputGain != 1.0f)
	{
		UE_LOGF(LogMicManager, Log, "Scaling gain of recording by %.2f linear gain.", InputGain);

		int32 ChannelIndex = InSourceSettings.InputChannelNumber - 1;
		int16* RawData = DeinterleavedAudio[ChannelIndex].PCMData.GetData();

		for (int32 SampleIndex = 0; SampleIndex < InNumFrames; ++SampleIndex)
		{
			// Scale by input gain, clamp to prevent integer overflow when casting back to int16. Will still clip.
			RawData[SampleIndex] = (int16)FMath::Clamp(InputGain * (float)RawData[SampleIndex], -32767.0f, 32767.0f);
		}
	}
}

TObjectPtr<USoundWave> FAudioRecordingManager::CreateSoundWaveAsset(const FRecordingManagerSourceSettings& InSourceSettings)
{
	uint8* RawData = nullptr;
	int32 NumChannelsToSerialize = 1;
	int32 NumWavesToSerialize = NumInputChannels;

	TObjectPtr<USoundWave> NewSoundWave = nullptr;
	USoundWave* ExistingSoundWave = nullptr;

	TArray<UAudioComponent*> ComponentsToRestart;
	bool bCreatedPackage = false;

	int32 NumBytes = 0;

	// De-interleaved buffer size will be the number of frames of audio
	int32 NumFrames = NumRecordedSamples / NumInputChannels;

	// This is the num bytes per de-interleaved audio buffer
	NumBytes = NumFrames * sizeof(int16);

	ApplyGain(InSourceSettings, NumFrames);

	if (InSourceSettings.Directory.Path.IsEmpty() || InSourceSettings.AssetName.IsEmpty())
	{
		// Create a new sound wave object from the transient package
		NewSoundWave = NewObject<USoundWave>(GetTransientPackage(), *InSourceSettings.AssetName);
	}
	else
	{
		FString PackageName;
		FString AssetName = InSourceSettings.AssetName;
		PackageName = InSourceSettings.Directory.Path / AssetName;

		int32 ChannelIndex = InSourceSettings.InputChannelNumber - 1;
		// Get the raw data from the deinterleaved audio location.
		RawData = (uint8*)DeinterleavedAudio[ChannelIndex].PCMData.GetData();

		UPackage* Package = CreatePackage(*PackageName);

		// Create a raw .wav file to stuff the raw PCM data in so when we create the sound wave asset it's identical to a normal imported asset
		check(RawData != nullptr);
		SerializeWaveFile(RawWaveData, RawData, NumBytes, NumChannelsToSerialize, WAVE_FILE_SAMPLERATE);

		// Check to see if a sound wave already exists at this location
		ExistingSoundWave = FindObject<USoundWave>(Package, *AssetName);

		// See if it's currently being played right now
		FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();
		if (AudioDeviceManager && ExistingSoundWave)
		{
			AudioDeviceManager->StopSoundsUsingResource(ExistingSoundWave, &ComponentsToRestart);
		}

		// Create a new sound wave object
		if (ExistingSoundWave)
		{
			NewSoundWave = ExistingSoundWave;
			NewSoundWave->FreeResources();
		}
		else
		{
			NewSoundWave = NewObject<USoundWave>(Package, *AssetName, RF_Public | RF_Standalone);
		}

		// Compressed data is now out of date.
		NewSoundWave->InvalidateCompressedData(true, false);

		// Copy the raw wave data file to the sound wave for storage. Will allow the recording to be exported.
		FSharedBuffer UpdatedBuffed = FSharedBuffer::Clone(RawWaveData.GetData(), RawWaveData.Num());
		NewSoundWave->RawData.UpdatePayload(UpdatedBuffed);

		bCreatedPackage = true;
	}

	if (NewSoundWave)
	{
		// Copy the recorded data to the sound wave so we can quickly preview it
		NewSoundWave->RawPCMDataSize = NumBytes;
		NewSoundWave->RawPCMData = (uint8*)FMemory::Malloc(NewSoundWave->RawPCMDataSize);
		FMemory::Memcpy(NewSoundWave->RawPCMData, RawData, NumBytes);

		// Calculate the duration of the sound wave
		// Note: We use the NumInputChannels for duration calculation since NumChannelsToSerialize may be 1 channel while NumInputChannels is 2 for the "split stereo" feature.
		NewSoundWave->Duration = (float)(NumRecordedSamples / NumInputChannels) / WAVE_FILE_SAMPLERATE;
		NewSoundWave->SetSampleRate(WAVE_FILE_SAMPLERATE);
		NewSoundWave->NumChannels = NumChannelsToSerialize;
		NewSoundWave->SetSoundAssetCompressionType(ESoundAssetCompressionType::BinkAudio);
		NewSoundWave->SetTimecodeInfo(GetTimecodeInfo(InSourceSettings));

		// Post import initializes USoundWave internal data structures 
		NewSoundWave->PostImport();

		if (bCreatedPackage)
		{
			FAssetRegistryModule::AssetCreated(NewSoundWave);
			NewSoundWave->MarkPackageDirty();

			// Restart any audio components if they need to be restarted
			for (int32 ComponentIndex = 0; ComponentIndex < ComponentsToRestart.Num(); ++ComponentIndex)
			{
				ComponentsToRestart[ComponentIndex]->Play();
			}
		}

		RecordedSoundWaves[InSourceSettings.InputChannelNumber - 1] = NewSoundWave;
	}

	return NewSoundWave;
}

FSoundWaveTimecodeInfo FAudioRecordingManager::GetTimecodeInfo(const FRecordingManagerSourceSettings& InSourceSettings)
{
	FSoundWaveTimecodeInfo TimecodeInfo;

	double SecondsFromMidnight = InSourceSettings.StartTimecode.ToTimespan(InSourceSettings.VideoFrameRate).GetTotalSeconds();
	if (!ensureAlwaysMsgf(SecondsFromMidnight >= 0, TEXT("Invalid SecondsFromMidnight value: %f, defaulting to 0"), SecondsFromMidnight))
	{
		SecondsFromMidnight = 0.0;
	}

	TimecodeInfo.NumSamplesSinceMidnight = static_cast<uint64>(SecondsFromMidnight * WAVE_FILE_SAMPLERATE);
	TimecodeInfo.NumSamplesPerSecond = WAVE_FILE_SAMPLERATE;
	TimecodeInfo.Description = InSourceSettings.AssetName;
	TimecodeInfo.OriginatorDate = FDateTime::Now().ToString(TEXT("%Y-%m-%d"));
	TimecodeInfo.OriginatorTime = FDateTime::Now().ToString(TEXT("%H:%M:%S"));
	// Not using FApp::GetName() here because it gets the executable name on disk which can be modified.
	// This is similar to what the splash screen does.
	TimecodeInfo.OriginatorDescription = FString(TEXT("Unreal Editor"));
	TimecodeInfo.OriginatorReference = FEngineVersion::Current().ToString();
	TimecodeInfo.TimecodeRate = InSourceSettings.VideoFrameRate;
	TimecodeInfo.bTimecodeIsDropFrame = InSourceSettings.StartTimecode.bDropFrameFormat;

	// Copy elision
	return TimecodeInfo;
}

int32 FAudioRecordingManager::OnAudioCapture(const void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow)
{
	if (RecorderState == EAudioRecorderState::Recording && InBufferFrames > 0)
	{
		if (bOverflow)
		{
			++NumOverflowsDetected;
		}

		CurrentRecordedPCMData.Append((const int16*)InBuffer, InBufferFrames * NumInputChannels);
		return 0;
	}

	return 1;
}

bool FAudioRecordingManager::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 InDeviceIndex)
{
	return AudioCapture.GetCaptureDeviceInfo(OutInfo, InDeviceIndex);
}

bool FAudioRecordingManager::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32& OutDeviceIndex, FString InDeviceId)
{
	TArray<FCaptureDeviceInfo> DeviceInfoArray;
	if (GetCaptureDevicesAvailable(DeviceInfoArray))
	{
		for (int32 DeviceIndex = 0; DeviceIndex < DeviceInfoArray.Num(); ++DeviceIndex)
		{
			FCaptureDeviceInfo& DeviceInfo = DeviceInfoArray[DeviceIndex];
			if (DeviceInfo.DeviceId == InDeviceId)
			{
				OutInfo = DeviceInfo;
				OutDeviceIndex = DeviceIndex;

				return true;
			}
		}
	}

	return false;
}

bool FAudioRecordingManager::GetCaptureDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
{
	return AudioCapture.GetCaptureDevicesAvailable(OutDevices) != 0;
}

} // namespace Audio
