// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"
#include "Engine/EngineTypes.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "Sound/SoundWaveTimecodeInfo.h"

class USoundWave;

DECLARE_LOG_CATEGORY_EXTERN(LogMicManager, Log, All);

namespace Audio
{
	// Settings used for starting a recording with the recording manager
	struct FRecordingSettings
	{
		// The audio device to use for capturing
		FString AudioCaptureDeviceId;

		// The desired buffer size in samples.
		int32 AudioInputBufferSize = 1024;

		// The length of the recording. Set to 0.0 or less for unbounded recording duration.
		float RecordingDuration = -1.0f;

		// Number of channels to record 
		int32 NumRecordChannels = 0;
	};

	// Settings used for individual audio input sources
	struct FRecordingManagerSourceSettings
	{
		// The place to put the audio file recording
		FDirectoryPath Directory;

		// The name of the asset you want to record
		FString AssetName;

		// The output gain of the recording. Allows you to boost the amplitude of your recording.
		float GainDb = 0.0f;

		// The channel number of the selected audio device to use for recording.
		int32 InputChannelNumber = -1;

		// The timecode locations for this recording
		FTimecode StartTimecode;

		// The video framerate used during this recording
		FFrameRate VideoFrameRate;
	};

	/**
	 * FAudioRecordingManager
	 * Singleton Mic Recording Manager -- generates recordings, stores the recorded data and plays them back
	 */
	class FAudioRecordingManager
	{
	public:
		
		FAudioRecordingManager();

		~FAudioRecordingManager();

		// Returns true if this object is in the PreRecord state indicating it is ready to record
		bool IsReadyToRecord() { return RecorderState == EAudioRecorderState::PreRecord; }

		// Returns true if this object is in the Recording state indicating it is currently recording
		bool IsRecording() { return RecorderState == EAudioRecorderState::Recording; }

		// Returns true if this object is in the Stopped state indicating it has finished recording
		bool IsStopped() { return RecorderState == EAudioRecorderState::Stopped; }

		// Starts a new recording with the given name and optional duration. 
		// If set to -1.0f, a duration won't be used and the recording length will be determined by StopRecording().
		void StartRecording(const FRecordingSettings& Settings);

		// Stops recording if the recording manager is recording. If not recording but has recorded data (due to set duration), it will just return the generated USoundWave.
		void StopRecording();

		// Called by RtAudio when a new audio buffer is ready to be supplied.
		int32 OnAudioCapture(const void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow);

		TObjectPtr<USoundWave> GetRecordedSoundWave(const FRecordingManagerSourceSettings& InSourceSettings);

		// Returns the audio capture device information at the given index.
		bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 InDeviceIndex = INDEX_NONE);

		// Returns the total amount of audio devices.
		bool GetCaptureDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices);

	private:

		// States used by AudioRecordingManager state machine
		enum class EAudioRecorderState : uint8
		{
			PreRecord,
			Recording,
			Stopped,
			ErrorDetected
		};

		// Called after recording to handle processing of recorded data (SRC, de-interleaving, apply gain, etc.)
		void ProcessRecordedData(const FRecordingManagerSourceSettings& InSourceSettings);

		// Deinterleave recorded data into discreet channels
		void DeinterleaveRecordedData();

		// Applies the user supplied gain parameter if neccessary to a single channel of audio
		void ApplyGain(const FRecordingManagerSourceSettings& InSourceSettings, const int32 InNumFrames);

		// Creates a wave asset for a single channel of audio data
		TObjectPtr<USoundWave> CreateSoundWaveAsset(const FRecordingManagerSourceSettings& InSourceSettings);

		// Returns the device info and index for a given device id
		bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32& OutDeviceIndex, FString InDeviceId);

		// Returns the timecode info for this recording
		FSoundWaveTimecodeInfo GetTimecodeInfo(const FRecordingManagerSourceSettings& InSourceSettings);

		// The default mic sample rate
		const int32 WAVE_FILE_SAMPLERATE = 44100;

		// Current state for the state machine
		std::atomic<EAudioRecorderState> RecorderState;

		// Copy of the input recording settings
		FRecordingSettings Settings;

		// FAudioCapture object -- used to interact with low-level audio device.
		FAudioCapture AudioCapture;

		// The data which is currently being recorded to, if the manager is actively recording. This is not safe to access while recording.
		TArray<int16> CurrentRecordedPCMData;

		// Buffer to store sample rate converted PCM data
		TArray<int16> ConvertedPCMData;

		// Buffers to de-interleave recorded audio
		struct FDeinterleavedAudio
		{
			TArray<int16> PCMData;
		};
		TArray<FDeinterleavedAudio> DeinterleavedAudio;

		// Reusable raw wave data buffer to generate .wav file formats
		TArray<uint8> RawWaveData;

		// The number of frames that have been recorded
		int32 NumRecordedSamples;

		// The number of frames to record if recording a set duration
		int32 NumFramesToRecord;

		// Recording block size (number of frames per callback block)
		int32 RecordingBlockSize;

		// The sample rate used in the recording
		float RecordingSampleRate;

		// Num input channels
		int32 NumInputChannels;

		// Number of overflows detected while recording
		int32 NumOverflowsDetected;

		// Pointer to the recorded PCM data after any sample rate conversion has been applied
		TArray<int16>* PCMDataToSerialize = nullptr;

		// Array of USoundWaves created during this record pass
		TArray<TObjectPtr<USoundWave>> RecordedSoundWaves;
	};

}
