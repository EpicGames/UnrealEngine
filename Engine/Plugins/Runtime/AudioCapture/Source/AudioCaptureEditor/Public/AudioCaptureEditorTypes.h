// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

/** Sequencer version of the audio device info. */
struct FTakeRecorderAudioDeviceInfo
{
	/** The name for this audio device*/
	FString DeviceName;
	/** The unique identifier for this device */
	FString DeviceId;
	/** The number of input channels this device is capable of capturing */
	int32 InputChannels = 0;
	/** The preferred sample rate for this device */
	int32 PreferredSampleRate = 0;
};

/** Take Recorder version of the audio recording settings. */
struct FTakeRecorderAudioSettings
{
	/** Audio device to use for capture. */
	FString AudioCaptureDeviceId;

	/** The desired buffer size in samples. */
	int32 AudioInputBufferSize = 1024;

	/** Optional audio recording duration in seconds. */
	float RecordingDuration = -1.0f;

	/** Number of channels to record */
	int32 NumRecordChannels = 0;
};

/** Take Recorder version of the audio source settings. */
struct FTakeRecorderAudioSourceSettings
{
	/** Directory to create the asset within (empty for transient package) */
	FDirectoryPath Directory;

	/** Name of the asset. */
	FString AssetName;

	/** Gain in decibels of the output asset. */
	float GainDb = 0.0f;

	/** The channel number of the selected audio device to use for recording. */
	int32 InputChannelNumber = -1;

	/** The timecode position for this recording */
	FTimecode StartTimecode;

	/** The video frame rate used during this recording */
	FFrameRate VideoFrameRate;
};
