// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureEditorTypes.h"
#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"

class USoundWave;

class IAudioCaptureEditor
{
public:

	/** Virtual destructor */
	virtual ~IAudioCaptureEditor() = default;

	/** Returns true if this object is in the PreRecord state indicating it is ready to record */
	virtual bool IsReadyToRecord() = 0;

	/** Returns true if this object is in the Recording state indicating it is currently recording */
	virtual bool IsRecording() = 0;

	/** Returns true if this object is in the Stopped state indicating it has finished recording */
	virtual bool IsStopped() = 0;

	/** Start recording audio data */
	virtual void Start(const FTakeRecorderAudioSettings& InSettings) = 0;

	/** Stop recording audio data */
	virtual void Stop() = 0;

	/** Returns previously recorded audio data for single channel as a USoundWave */
	virtual TObjectPtr<USoundWave> GetRecordedSoundWave(const FTakeRecorderAudioSourceSettings& InSourceSettings) = 0;

	/** Returns the audio capture device information at the given index */
	virtual bool GetCaptureDeviceInfo(FTakeRecorderAudioDeviceInfo& OutInfo, int32 DeviceIndex = INDEX_NONE) = 0;

	/** Returns the total amount of audio devices */
	virtual bool GetCaptureDevicesAvailable(TArray<FTakeRecorderAudioDeviceInfo>& OutDevices) = 0;
};
