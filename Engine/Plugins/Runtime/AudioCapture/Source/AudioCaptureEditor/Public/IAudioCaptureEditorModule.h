// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAudioCaptureEditor.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

class ITakeRecorderAudio;

class IAudioCaptureEditorModule : public IModuleInterface
{
public:

	/**
	 * Register a function that will return a new audio capturer for the specified parameters
	 * @param	FactoryFunction		Function used to generate a new audio recorder
	 * @return A handle to be passed to UnregisterAudioRecorder to unregister the recorder
	 */
	virtual FDelegateHandle RegisterAudioRecorder(const TFunction<TUniquePtr<IAudioCaptureEditor>()>& FactoryFunction) = 0;

	/**
	 * Unregister a previously registered audio recorder factory function
	 * @param	RegisteredHandle	The handle returned from RegisterAudioRecorder
	 */
	virtual void UnregisterAudioRecorder(FDelegateHandle RegisteredHandle) = 0;

	/**
	 * Check whether we have an audio recorder registered or not
	 * @return true if we have an audio recorder registered, false otherwise
	 */
	virtual bool HasAudioRecorder() const = 0;

	/**
	 * Attempt to create an audio recorder
	 * @param	Settings	Settings for the audio recorder
	 * @return A valid ptr to an audio recorder or null
	 */
	virtual TUniquePtr<IAudioCaptureEditor> CreateAudioRecorder() const = 0;
};
